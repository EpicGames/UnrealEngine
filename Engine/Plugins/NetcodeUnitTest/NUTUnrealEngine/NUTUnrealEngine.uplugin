{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Netcode Unit Test - Unreal Engine",
	"Description" : "Exploit unit tests for Unreal Engine and some base Unreal Engine games, based on the Netcode Unit Test framework",
	"Category" : "Networking",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "NUTUnrealEngine",
			"Type" : "UncookedOnly",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Win64",
				"Linux"
			]
		}
	],
	"Plugins":
	[
		{
			"Name": "NetcodeUnitTest",
			"Enabled": true
		}
	]
}
