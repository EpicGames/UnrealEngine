// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "ClientUnitTest.h"

#include "FTextCrash.generated.h"

/**
 * Tests an RPC crash caused by empty FText's, as reported on the UDN here:
 * https://udn.unrealengine.com/questions/213120/using-empty-ftexts-within-rpcs.html
 *
 * UDN Post: "Using Empty FTexts within RPCs"
 * Hey,
 * we're using FTexts within RPCs functions (server -> client in my specific case) to pass localized strings.
 * That works fine until the point when the server sends an empty FText.
 * In that case both the FText members SourceString and DisplayString are null on client side which lead to crashes whenever you use
 * something like ToString which assumes those are valid.
 *
 * Is this the intended behavior? I'm using FTextInspector::GetSourceString(text) to run checks on these replicated FTexts now to catch
 * this case. FTexts that are not empty work just fine.
 *
 * Thanks, Oliver
 */
UCLASS()
class UFTextCrash : public UClientUnitTest
{
	GENERATED_UCLASS_BODY()

public:
	virtual void InitializeEnvironmentSettings() override;

	virtual void ExecuteClientUnitTest() override;

	virtual void NotifyProcessLog(TWeakPtr<FUnitTestProcess> InProcess, const TArray<FString>& InLogLines) override;
};

