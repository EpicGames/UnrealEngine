// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "UnitTest.h"

#include "NetBitsTest.generated.h"


/**
 * @todo JohnB
 */
UCLASS()
class UNetBitsTest : public UUnitTest
{
	GENERATED_UCLASS_BODY()

public:
	virtual bool ExecuteUnitTest() override;
};
