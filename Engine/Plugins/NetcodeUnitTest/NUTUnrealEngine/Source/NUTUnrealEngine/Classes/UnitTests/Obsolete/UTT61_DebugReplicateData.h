// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/ObjectPtr.h"
#include "ClientUnitTest.h"

#include "UTT61_DebugReplicateData.generated.h"


// Forward declarations
class AActor;
class UActorChannel;


/**
 * Implements UTT GameplayDebuggingComponent/ServerReplicateData crash
 *
 * Command:
 *	UTT -b 61 127.0.0.1
 *
 * Documentation from Luigi:
 * - TEST VULNERABILITY: ServerReplicateData array overflow
 * ####
 * 07 May 2014
 * ServerReplicateData array overflow
 * 
 * The ServerReplicateMessageToAIDebugView function used for some
 * debugging features is affected by an array overflow with the
 * InMessages: ActivateDataView and DeactivateDataView.
 * 
 * This is the same function that was affected by the bug 55 reported
 * the 27 March, anyway remember that it's NOT used in shipping builds.
 */
UCLASS()
class UUTT61_DebugReplicateData : public UClientUnitTest
{
	GENERATED_UCLASS_BODY()

protected:
	/** Reference to the GameplayDebuggingReplicator class */
	UPROPERTY()
	TObjectPtr<UClass> RepClass;

private:
	/** Once the GameplayDebuggingReplicator is received, cache it */
	UPROPERTY()
	TWeakObjectPtr<AActor> Replicator;

public:
	virtual void InitializeEnvironmentSettings() override;

	virtual void NotifyAllowNetActor(UClass* ActorClass, bool bActorChannel, bool& bBlockActor);

	virtual void NotifyNetActor(UActorChannel* ActorChannel, AActor* Actor);

	virtual void ExecuteClientUnitTest() override;

	virtual void NotifyProcessLog(TWeakPtr<FUnitTestProcess> InProcess, const TArray<FString>& InLogLines) override;
};
