// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


// @todo #JohnB: Restore in a game-level package, eventually
#if 0
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ClientUnitTest.h"

#include "PacketLimitTest.generated.h"

struct FUniqueNetIdRepl;

/**
 * The current stage of testing for the unit test
 */
enum class ELimitTestStage : uint8
{
	LTS_LowLevel_AtLimit	= 0,
	LTS_LowLevel_OverLimit	= 1,
	LTS_Bunch_AtLimit		= 2,
	LTS_Bunch_OverLimit		= 3,

	LTS_MAX					= 4
};

/**
 * Used to get name values for the above enum
 */
inline FString GetLimitTestStageName(ELimitTestStage Stage)
{
	#define ELTS_CASE(x) case ELimitTestStage::x : return TEXT(#x)

	switch (Stage)
	{
		ELTS_CASE(LTS_LowLevel_AtLimit);
		ELTS_CASE(LTS_LowLevel_OverLimit);
		ELTS_CASE(LTS_Bunch_AtLimit);
		ELTS_CASE(LTS_Bunch_OverLimit);
		ELTS_CASE(LTS_MAX);

	default:
		return FString::Printf(TEXT("Unknown 0x%08X"), (uint32)Stage);
	}

	#undef ELTS_CASE
}

/**
 * Unit test for testing packet bit-size limits, and ensuring edge cases don't trigger warnings or send failures
 */
UCLASS()
class UPacketLimitTest : public UClientUnitTest
{
	GENERATED_UCLASS_BODY()

protected:
	/** Whether or not to test with Oodle enabled. */
	bool bUseOodle;

	/** The current stage of testing */
	ELimitTestStage TestStage;

	/** The last attempted send size passed to the connection socket (after PacketHandlers) */
	int32 LastSocketSendSize;

	/** The size the final packet needs to be, if it is to be sent - all packets not matching this size, are blocked */
	int32 TargetSocketSendSize;

public:
	virtual void InitializeEnvironmentSettings() override;

	virtual bool ValidateUnitTestSettings(bool bCDOCheck) override;

	// @todo #JohnB: Remove
	//virtual bool ConnectMinimalClient(FUniqueNetIdRepl* InNetID) override;

	virtual void ExecuteClientUnitTest() override;

	virtual void NotifySendRawPacket(void* Data, int32 Count, bool& bBlockSend) override;

	virtual void NotifySocketSendRawPacket(void* Data, int32 Count, bool& bBlockSend) override;

	virtual void NotifyProcessLog(TWeakPtr<FUnitTestProcess> InProcess, const TArray<FString>& InLogLines) override;

protected:
	/**
	 * Advanced to the next test stage
	 */
	void NextTestStage();
};

#endif
