// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class NUTUnrealEngine : ModuleRules
	{
		public NUTUnrealEngine(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange
			(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"NetcodeUnitTest"
				}
			);

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;
		}
	}
}
