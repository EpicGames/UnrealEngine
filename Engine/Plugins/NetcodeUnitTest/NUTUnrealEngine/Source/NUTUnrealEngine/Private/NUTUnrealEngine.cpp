// Copyright Epic Games, Inc. All Rights Reserved.

// Includes
#include "INUTUnrealEngine.h"

#include "UnrealEngineEnvironment.h"

/**
 * Module implementation
 */
class FNUTUnrealEngine : public INUTUnrealEngine
{
public:
	virtual void StartupModule() override
	{
		FNUTModuleInterface::StartupModule();

		FShooterGameEnvironment::Register();
		FQAGameEnvironment::Register();
		FUTEnvironment::Register();
	}

	virtual void ShutdownModule() override
	{
		FNUTModuleInterface::ShutdownModule();
	}
};


IMPLEMENT_MODULE(FNUTUnrealEngine, NUTUnrealEngine);

