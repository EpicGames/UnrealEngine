// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnitTests/NetBitsTest.h"
#include "Engine/NetConnection.h"
#include "NetcodeUnitTest.h"



/**
 * UNetBitsTest
 */

UNetBitsTest::UNetBitsTest(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	UnitTestName = TEXT("NetBitsTest");
	UnitTestType = TEXT("Test");

	UnitTestDate = FDateTime(2016, 03, 19);

	UnitTestTimeout = 60;

	bWorkInProgress = true;

	// @todo #JohnBExploitCL: Bugtracking/changelist notes

	ExpectedResult.Add(TEXT("ShooterGame"), EUnitTestVerification::VerifiedFixed);
}

bool UNetBitsTest::ExecuteUnitTest()
{
	bool bSuccess = true;

	TMap<FString, bool> TestResults;

	uint32 ReadValue = 0;

	// Invalid test - this triggers an assert
#if 0
	// SerializeInt, Zero Value, Zero Range
	{
		FBitWriter Writer(0, true);
		uint32 WriteValue = 0;

		Writer.SerializeInt(WriteValue, 0);

		TestResults.Add(TEXT("Zero range write"), Writer.GetNumBits() == 0);
	}
#endif

	// Invalid test - this triggers an assert
#if 0
	// ..., Zero Value, 1 range
	{
		FBitWriter Writer(0, true);
		uint32 WriteValue = 0;

		Writer.SerializeInt(WriteValue, 1);

		TestResults.Add(TEXT("One range write"), Writer.GetNumBits() == 1);
	}
#endif

	// SerializeInt, Zero Value, 2 range
	{
		FBitWriter Writer(0, true);
		uint32 WriteValue = 0;

		Writer.SerializeInt(WriteValue, 2);

		TestResults.Add(TEXT("Two range write"), !Writer.IsError() && Writer.GetNumBits() == 1);


		FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

		Reader.SerializeInt(ReadValue, 2);

		TestResults.Add(TEXT("Two range read"), !Reader.IsError() && ReadValue == WriteValue);
	}

	// SerializeInt, 1 Value, 2 range
	{
		FBitWriter Writer(0, true);
		uint32 WriteValue = 1;

		Writer.SerializeInt(WriteValue, 2);

		TestResults.Add(TEXT("One Value, Two range write"), !Writer.IsError() && Writer.GetNumBits() == 1);


		FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

		Reader.SerializeInt(ReadValue, 2);

		TestResults.Add(TEXT("One Value, Two range read"), !Reader.IsError() && ReadValue == WriteValue);
	}

	// Invalid test - this triggers an assert
#if 0
	// SerializeInt, 2 Value, 2 range (deliberate fail)
	{
		FBitWriter Writer(0, true);
		uint32 WriteValue = 2;

		Writer.SerializeInt(WriteValue, 2);

		TestResults.Add(TEXT("Two Value, Two range write (FAILURE)"), !Writer.IsError() && Writer.GetNumBits() == 1);


		FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

		Reader.SerializeInt(ReadValue, 2);

		TestResults.Add(TEXT("Two Value, Two range read (FAILURE)"), !Reader.IsError() && ReadValue == 1);
	}
#endif


	// SerializeInt, 0 Value, 3 range
	{
		FBitWriter Writer(0, true);
		uint32 WriteValue = 0;

		Writer.SerializeInt(WriteValue, 3);

		TestResults.Add(TEXT("Zero Value, Three range write"), !Writer.IsError() && Writer.GetNumBits() == 2);


		FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

		Reader.SerializeInt(ReadValue, 3);

		TestResults.Add(TEXT("Zero Value, Three range read"), !Reader.IsError() && ReadValue == WriteValue);
	}

	// SerializeInt, 1 Value, 3 range
	{
		FBitWriter Writer(0, true);
		uint32 WriteValue = 1;

		Writer.SerializeInt(WriteValue, 3);

		TestResults.Add(TEXT("One Value, Three range write"), !Writer.IsError() && Writer.GetNumBits() == 1);


		FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

		Reader.SerializeInt(ReadValue, 3);

		TestResults.Add(TEXT("One Value, Three range read"), !Reader.IsError() && ReadValue == WriteValue);
	}

	// SerializeInt, 2 Value, 3 range
	{
		FBitWriter Writer(0, true);
		uint32 WriteValue = 2;

		Writer.SerializeInt(WriteValue, 3);

		TestResults.Add(TEXT("Two Value, Three range write"), !Writer.IsError() && Writer.GetNumBits() == 2);


		FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

		Reader.SerializeInt(ReadValue, 3);

		TestResults.Add(TEXT("Two Value, Three range read"), !Reader.IsError() && ReadValue == WriteValue);
	}

	// Invalid test - this triggers an assert
#if 0
	// SerializeInt, 3 Value, 3 range (deliberate fail - technically enough bits to fit though)
	{
		FBitWriter Writer(0, true);
		uint32 WriteValue = 3;

		Writer.SerializeInt(WriteValue, 3);

		TestResults.Add(TEXT("Three Value, Three range write (FAILURE)"), !Writer.IsError() && Writer.GetNumBits() == 2);


		FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

		Reader.SerializeInt(ReadValue, 3);

		TestResults.Add(TEXT("Three Value, Three range read (FAILURE)"), !Reader.IsError() && ReadValue == 2);
	}
#endif

	// SerializeInt, 0 Value, 4294967295U range
	{
		FBitWriter Writer(0, true);
		uint32 WriteValue = 0;

		Writer.SerializeInt(WriteValue, 4294967295U);

		TestResults.Add(TEXT("Zero Value, Max uint32 range write"), !Writer.IsError() && Writer.GetNumBits() == 32);


		FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

		Reader.SerializeInt(ReadValue, 4294967295U);

		TestResults.Add(TEXT("Zero Value, Max uint32 range read"), !Reader.IsError() && ReadValue == WriteValue);
	}

	// SerializeInt, 4294967294U Value, 4294967295U range
	{
		FBitWriter Writer(0, true);
		uint32 WriteValue = 4294967294U;

		Writer.SerializeInt(WriteValue, 4294967295U);

		TestResults.Add(TEXT("Max uint32 Value minus 1, Max uint32 range write"), !Writer.IsError() && Writer.GetNumBits() == 32);


		FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

		Reader.SerializeInt(ReadValue, 4294967295U);

		TestResults.Add(TEXT("Max uint32 Value minus 1, Max uint32 range read"), !Reader.IsError() && ReadValue == WriteValue);
	}

	// Invalid test - this triggers an assert
#if 0
	// SerializeInt, 4294967295U Value, 4294967295U range (edge case that fails - impossible to send 4294967295U, even though it fits)
	{
		FBitWriter Writer(0, true);
		uint32 WriteValue = 4294967295U;

		Writer.SerializeInt(WriteValue, 4294967295U);

		TestResults.Add(TEXT("Max uint32 Value, Max uint32 range write (FAILURE)"), !Writer.IsError() && Writer.GetNumBits() == 32);


		FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

		Reader.SerializeInt(ReadValue, 4294967295U);

		TestResults.Add(TEXT("Max uint32 Value, Max uint32 range read (FAILURE)"), !Reader.IsError() && ReadValue == WriteValue - 1);
	}
#endif

	// SerializeIntPacked, different values at different offsets
	{
		uint32 Padding = 0xFFFFFFFFU;
		FBitWriter Writer(256, false);

		const uint32 TestValues[] = { 0U, 0x43U, 0x1234U, 0x12345U, 0x7654321U, 0xFFFFFFFFU };
		const size_t TestValueCount = sizeof(TestValues) / sizeof(TestValues[0]);
		for (uint32 BitOffset = 0; BitOffset <= 32; ++BitOffset)
		{
			for (size_t TestIt = 0; TestIt != TestValueCount; ++TestIt)
			{
				uint32 TestValue = TestValues[TestIt];
				Writer.Reset();
				Writer.SerializeBits(&Padding, BitOffset);
				Writer.SerializeIntPacked(TestValue);

				uint32 ReadPadding = 0;
				FBitReader Reader(Writer.GetData(), Writer.GetNumBits());
				Reader.SerializeBits(&ReadPadding, BitOffset);
				Reader.SerializeIntPacked(ReadValue);
				if (BitOffset == 0 || ReadValue != TestValue)
					TestResults.Add(FString::Printf(TEXT("SerializeIntPacked 0x%08X, got 0x%08X with bit offset %u"), TestValue, ReadValue, BitOffset), !Writer.IsError() && ReadValue == TestValue);
			}
		}
	}

	// @todo #JohnB: Remove or incorporate properly - size check
#if 0
	{
		for (uint32 MaxValue=3; MaxValue<9; MaxValue++)
		{
			for (uint32 WriteValue=0; WriteValue<MaxValue; WriteValue++)
			{
				FBitWriter Writer(0, true);

				Writer.SerializeInt(WriteValue, MaxValue);

				UNIT_LOG(, TEXT("Max: %u, Write: %u, NumBits: %u"), MaxValue, WriteValue, Writer.GetNumBits());


				FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

				uint32 ReadValue = 0;

				Reader.SerializeInt(ReadValue, MaxValue);

				if (ReadValue == WriteValue)
				{
					UNIT_LOG(, TEXT("Match"));
				}
				else
				{
					UNIT_LOG(, TEXT("FAILED MATCH"));
				}
			}
		}
	}
#endif

	// @todo #JohnB: REmove or incorporate properly - checking MAX_PACKETID size limits
	{
		int64 MaxBits = 0;

		for (uint32 WriteValue=0; WriteValue<MAX_PACKETID; WriteValue++)
		{
			FBitWriter Writer(0, true);

			Writer.WriteIntWrapped(WriteValue, MAX_PACKETID);

			if (Writer.GetNumBits() > MaxBits)
			{
				MaxBits = Writer.GetNumBits();
			}
		}

		UNIT_LOG(, TEXT("MaxBits: %lli"), MaxBits);
	}


	// @todo #JohnB: Remove or incorporate properly - size check
#if 0
	{
		UNIT_LOG(, TEXT("SerializeIntMax test"));

		for (uint32 WriteValue=0; WriteValue<=MAX_PACKET_SIZE; WriteValue++)
		{
			FBitWriter Writer(0, true);

			Writer.SerializeIntMax(WriteValue, MAX_PACKET_SIZE);

			UNIT_LOG(, TEXT("Write: %u, NumBits: %u"), WriteValue, Writer.GetNumBits());


			FBitReader Reader(Writer.GetData(), Writer.GetNumBits());

			uint32 ReadValue = 0;

			Reader.SerializeIntMax(ReadValue, MAX_PACKET_SIZE);

			if (ReadValue == WriteValue)
			{
				UNIT_LOG(, TEXT("Match"));
			}
			else
			{
				UNIT_LOG(, TEXT("FAILED MATCH"));
			}
		}
	}
#endif

	// @todo #JohnB: Use this to test the performance of the bit writer code as well, and to test optimizations to its performance

	// @todo #JohnB: List of things to optimize performance-wise:
	//					- Change SerializeInt/WriteIntWrapped, to optimize-away the log2 function where possible (research this)
	//					- appBitscpy is extremely expensive. Optimize this as best you can (why store bits that way anyway? odd...)


	// Verify the results
	for (TMap<FString, bool>::TConstIterator It(TestResults); It; ++It)
	{
		UNIT_LOG(ELogType::StatusImportant, TEXT("Test '%s' returned: %s"), *It.Key(), (It.Value() ? TEXT("Success") : TEXT("FAIL")));

		if (!It.Value())
		{
			VerificationState = EUnitTestVerification::VerifiedNeedsUpdate;
		}
	}


	if (VerificationState == EUnitTestVerification::Unverified)
	{
		VerificationState = EUnitTestVerification::VerifiedFixed;
	}

	return bSuccess;
}
