// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnitTests/PacketLimitTest.h"

// @todo #JohnB: Restore in a game-level package, eventually
#if 0
#include "EngineGlobals.h"
#include "Engine/Engine.h"
#include "Sockets.h"
#include "Engine/NetConnection.h"

// @todo #JohnB: Remove when migrated
//#include "IpConnection.h"

#include "MinimalClient.h"
#include "UnitTestEnvironment.h"

#include "Net/NUTUtilNet.h"
#include "Net/UnitTestNetConnection.h"



/**
 * UPacketLimitTest
 */

UPacketLimitTest::UPacketLimitTest(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bUseOodle(false)
	, TestStage(ELimitTestStage::LTS_LowLevel_AtLimit)
	, LastSocketSendSize(0)
	, TargetSocketSendSize(0)
{
	UnitTestName = TEXT("PacketLimitTest");
	UnitTestType = TEXT("Test");

	UnitTestDate = FDateTime(2015, 12, 23);

	bWorkInProgress = true;

	// @todo #JohnBExploitCL: Bugtracking/changelist notes

	ExpectedResult.Add(TEXT("ShooterGame"), EUnitTestVerification::VerifiedNotFixed);

	UnitTestTimeout = 60;


	SetFlags<EUnitTestFlags::LaunchServer | EUnitTestFlags::AutoReconnect | EUnitTestFlags::RequirePing |
				EUnitTestFlags::CaptureSendRaw,
				EMinClientFlags::SkipControlJoin>();
}

void UPacketLimitTest::InitializeEnvironmentSettings()
{
	BaseServerURL = UnitEnv->GetDefaultMap(UnitTestFlags);
	BaseServerParameters = UnitEnv->GetDefaultServerParameters();

	if (bUseOodle)
	{
		BaseServerURL += TEXT(" -PacketHandler -Oodle");
	}
}

bool UPacketLimitTest::ValidateUnitTestSettings(bool bCDOCheck)
{
	bool bReturnVal = Super::ValidateUnitTestSettings(bCDOCheck);

#if UE_BUILD_SHIPPING
	UNIT_LOG(ELogType::StatusFailure, TEXT("The 'PacketLimitTest' unit test, can only be run in non-shipping mode."));

	bReturnVal = false;
#endif

	return bReturnVal;
}

// @todo #JohnB: Remove
#if 0
bool UPacketLimitTest::ConnectMinimalClient(FUniqueNetIdRepl* InNetID)
{
	bool bReturnVal = false;
	bool bForceEnableDefault = UUnitTestNetConnection::bForceEnableHandler;

	if (bUseOodle)
	{
		UUnitTestNetConnection::bForceEnableHandler = true;
		GEngine->Exec(NULL, TEXT("Oodle ForceEnable On"));
	}

	bReturnVal = Super::ConnectMinimalClient(InNetID);

	if (bUseOodle)
	{
		GEngine->Exec(NULL, TEXT("Oodle ForceEnable Default"));
		UUnitTestNetConnection::bForceEnableHandler = bForceEnableDefault;
	}


	return bReturnVal;
}
#endif

void UPacketLimitTest::ExecuteClientUnitTest()
{
	bool bLowLevelSend = TestStage == ELimitTestStage::LTS_LowLevel_AtLimit || TestStage == ELimitTestStage::LTS_LowLevel_OverLimit;
	bool bBunchSend = TestStage == ELimitTestStage::LTS_Bunch_AtLimit || TestStage == ELimitTestStage::LTS_Bunch_OverLimit;

	// You can't access LowLevelSend from UNetConnection, but you can from UIpConnection, as it's exported there
	UIpConnection* IpConn = (MinClient != nullptr ? Cast<UIpConnection>(MinClient->GetConn()) : nullptr);

	if (IpConn != nullptr)
	{
		int32 PacketLimit = IpConn->MaxPacket;
		int32 SocketLimit = IpConn->MaxPacket;
		TArray<uint8> PacketData;

		if (bBunchSend)
		{
			int64 FreeBits = IpConn->SendBuffer.GetMaxBits() - MAX_BUNCH_HEADER_BITS + MAX_PACKET_TRAILER_BITS;

			PacketLimit = FreeBits / 8;

			check(PacketLimit > 0)
		}

		if (TestStage == ELimitTestStage::LTS_LowLevel_OverLimit || TestStage == ELimitTestStage::LTS_Bunch_OverLimit)
		{
			// Nudge the packet over the MaxPacket limit (accurate for LowLevel, approximate for Bunch)
			PacketLimit++;
			SocketLimit++;
		}

		PacketData.AddZeroed(PacketLimit);

		// Randomize the packet data (except for last byte), to thwart any compression, which would cause infinite recursion below
		// (e.g. recursively adding just zero's, would mean the same compressed size almost every time)
		for (int32 i=0; i<PacketData.Num()-1; i++)
		{
			PacketData[i] = FMath::Rand() % 255;
		}


		// Iteratively run 'test' sends, where the packet is passed through all the netcode but not sent,
		// unless the final (post-PacketHandler) packet size matches SocketLimit.
		bool bPacketAtLimit = false;
		int32 TryCount = 0;
		int32 SendDelta = 0;

		// Blocks all socket sends not matching SocketLimit
		TargetSocketSendSize = SocketLimit;

		while (!bPacketAtLimit && TryCount < 16)
		{
			if (bLowLevelSend)
			{
				IpConn->LowLevelSend(PacketData.GetData(), PacketData.Num(), PacketData.Num() * 8);
			}
			else if (bBunchSend)
			{
				UUnitTestNetConnection* UnitTestConn = CastChecked<UUnitTestNetConnection>(IpConn);

				// If the bunch is to go over the limit, disable asserts
				bool bBunchOverLimit = TestStage == ELimitTestStage::LTS_Bunch_OverLimit;

				UnitTestConn->bDisableValidateSend = bBunchOverLimit;

				// @todo #JohnB: To re-enable this code above, you have to reimplement this now-removed UUnitTestNetConnection code:
#if 0
void UUnitTestNetConnection::ValidateSendBuffer()
{
	if (!bDisableValidateSend)
	{
		Super::ValidateSendBuffer();
	}
}
#endif

				IpConn->FlushNet();


				FOutBunch* TestBunch = MinClient->CreateChannelBunchByName(NAME_Control, 0);

				if (TestBunch != nullptr)
				{
					TestBunch->Serialize(PacketData.GetData(), PacketData.Num());

					IpConn->SendRawBunch(*TestBunch, false);
				}
				else
				{
					UNIT_LOG(ELogType::StatusFailure, TEXT("CreateChannelBunch failed - marking unit test as needing update."));

					VerificationState = EUnitTestVerification::VerifiedNeedsUpdate;
				}


				if (bBunchOverLimit)
				{
					// For a successful test, the bunch must cause a send error
					if (IpConn->SendBuffer.IsError())
					{
						bPacketAtLimit = true;

						UNIT_LOG(ELogType::StatusSuccess, TEXT("Detected successful bunch overflow. Moving to next test."));
						NextTestStage();
					}
					else
					{
						UNIT_LOG(ELogType::StatusFailure, TEXT("Failed to detect bunch overflow, when one was expected."));
						VerificationState = EUnitTestVerification::VerifiedNeedsUpdate;
					}

					break;
				}


				IpConn->FlushNet();

				UnitTestConn->bDisableValidateSend = false;
			}


			// If PacketHandlers have increased/decreased final packet size away from SocketLimit, trim/pad the packet and retry
			SendDelta = FMath::Max(1, (SendDelta == 0 ? FMath::Abs(LastSocketSendSize - SocketLimit) : SendDelta / 2));

			if (LastSocketSendSize > SocketLimit)
			{
				PacketData.RemoveAt(PacketData.Num()-1-SendDelta, SendDelta, false);
			}
			else if (LastSocketSendSize < SocketLimit)
			{
				PacketData.InsertZeroed(PacketData.Num()-1, SendDelta);

				for (int32 i=PacketData.Num()-1-SendDelta; i<SendDelta; i++)
				{
					PacketData[i] = FMath::Rand() % 255;
				}
			}
			else // if (LastSocketSendSize == SocketLimit)
			{
				// Packet successfully sent
				bPacketAtLimit = true;
			}

			TryCount++;
		}


		// Re-enable sending packets
		TargetSocketSendSize = 0;

		if (!bPacketAtLimit)
		{
			UNIT_LOG(ELogType::StatusFailure, TEXT("Failed to send packet - reached packet testing iteration limit."));
			VerificationState = EUnitTestVerification::VerifiedUnreliable;
		}
	}
}

void UPacketLimitTest::NotifySocketSendRawPacket(void* Data, int32 Count, bool& bBlockSend)
{
	LastSocketSendSize = Count;

	if (TargetSocketSendSize > 0)
	{
		if (Count == TargetSocketSendSize)
		{
			UNIT_LOG(, TEXT("Packet passed size filter of '%i' bytes."), TargetSocketSendSize);
		}
		else if (!bBlockSend)
		{
			bBlockSend = true;
		}
	}

	Super::NotifySocketSendRawPacket(Data, Count, bBlockSend);
}

void UPacketLimitTest::NotifyProcessLog(TWeakPtr<FUnitTestProcess> InProcess, const TArray<FString>& InLogLines)
{
	Super::NotifyProcessLog(InProcess, InLogLines);

	bool bMoveToNextStage = false;

	if (InProcess.HasSameObject(ServerHandle.Pin().Get()))
	{
		for (auto CurLine : InLogLines)
		{
			const bool bPacketOverflow = CurLine.Contains(TEXT("LogNet:Warning: UDP recvfrom error: 12 (SE_EMSGSIZE) from "));
			const bool bPacketReceived = CurLine.Contains(TEXT(" Malformed_Packet: Received packet with 0's in last byte of packet"));
			const bool bBadControlMsg = CurLine.Contains(TEXT("LogNet:Error: Received unknown control channel message"));

			bool bUnexpectedResult = false;

			if (TestStage == ELimitTestStage::LTS_LowLevel_AtLimit)
			{
				bUnexpectedResult = bPacketOverflow || bBadControlMsg;

				if (bPacketReceived)
				{
					UNIT_LOG(ELogType::StatusSuccess, TEXT("Detected successful packet send at limit. Moving to next test."));
					bMoveToNextStage = true;
				}
			}
			else if (TestStage == ELimitTestStage::LTS_LowLevel_OverLimit)
			{
				bUnexpectedResult = bPacketReceived || bBadControlMsg;

				if (bPacketOverflow)
				{
					UNIT_LOG(ELogType::StatusSuccess, TEXT("Detected successful packet overflow. Moving to next test."));
					bMoveToNextStage = true;
				}
			}
			else if (TestStage == ELimitTestStage::LTS_Bunch_AtLimit)
			{
				bUnexpectedResult = bPacketOverflow || bPacketReceived;

				if (bBadControlMsg)
				{
					UNIT_LOG(ELogType::StatusSuccess, TEXT("Detected successful bunch send at limit. Moving to next test."));
					bMoveToNextStage = true;
				}
			}


			if (bUnexpectedResult)
			{
				UNIT_LOG(ELogType::StatusFailure, TEXT("Detected unexpected log result for test stage '%s'."),
							*GetLimitTestStageName(TestStage));

				UNIT_LOG(ELogType::StatusFailure, TEXT("Values: bPacketOverflow: %i, bPacketReceived: %i, bBadControlMsg: %i"),
							(int32)bPacketOverflow, (int32)bPacketReceived, (int32)bBadControlMsg);
			}
		}
	}

	if (bMoveToNextStage)
	{
		NextTestStage();
	}
}

void UPacketLimitTest::NextTestStage()
{
	((uint8&)TestStage)++;

	if (TestStage >= ELimitTestStage::LTS_MAX)
	{
		UNIT_LOG(ELogType::StatusImportant, TEXT("Testing complete."));

		VerificationState = EUnitTestVerification::VerifiedFixed;
	}
	else
	{
		UNIT_LOG(, TEXT("Advancing TestStage to: %s"), *GetLimitTestStageName(TestStage));

		TriggerAutoReconnect();
	}
}
#endif
