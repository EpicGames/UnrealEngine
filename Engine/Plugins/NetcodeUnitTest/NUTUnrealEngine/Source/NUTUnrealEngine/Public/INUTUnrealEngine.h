// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Includes
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "INetcodeUnitTest.h"


class INUTUnrealEngine : public FNUTModuleInterface
{
public:
	INUTUnrealEngine()
		: FNUTModuleInterface(TEXT("NUTUnrealEngine"))
	{
	}

	static inline INUTUnrealEngine& Get()
	{
		return FModuleManager::LoadModuleChecked<INUTUnrealEngine>("NUTUnrealEngine");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("NUTUnrealEngine");
	}
};

